// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class EngineMessages : ModuleRules
	{
		public EngineMessages(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				});
		}
	}
}
