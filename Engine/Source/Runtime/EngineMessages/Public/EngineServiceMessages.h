// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "EngineServiceMessages.generated.h"


/* Service discovery messages
 *****************************************************************************/

/**
 * Implements a message for discovering engine instances on the network.
 */
USTRUCT()
struct FEngineServicePing
{
	GENERATED_USTRUCT_BODY()
};


/**
 * Implements a message for responding to a request to discover engine instances on the network.
 */
USTRUCT()
struct FEngineServicePong
{
	GENERATED_USTRUCT_BODY()

	/** Holds the name of the currently loaded level, if any. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString CurrentLevel;

	/** Holds the engine version. */
	UPROPERTY(EditAnywhere, Category="Message")
	int32 EngineVersion=0;

	/** Holds a flag indicating whether game play has begun. */
	UPROPERTY(EditAnywhere, Category="Message")
	bool HasBegunPlay=false;

	/** Holds the instance identifier. */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid InstanceId;

	/** Holds the type of the engine instance. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString InstanceType;

	/** Holds the identifier of the session that the application belongs to. */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid SessionId;

	/** Holds the time in seconds since the world was loaded. */
	UPROPERTY(EditAnywhere, Category="Message")
	float WorldTimeSeconds=0.0f;
};


/* Authorization messages
 *****************************************************************************/

/**
 * Implements a message for denying service access to a remote user.
 */
USTRUCT()
struct FEngineServiceAuthDeny
{
	GENERATED_USTRUCT_BODY()

	/** Holds the name of the user that denied access. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString UserName;

	/** Holds the name of the user that access is denied to. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString UserToDeny;
};


/**
 * Implements a message for granting service access to a remote user.
 */
USTRUCT()
struct FEngineServiceAuthGrant
{
	GENERATED_USTRUCT_BODY()

	/** Holds the name of the user that granted access. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString UserName;

	/** Holds the name of the user that access is granted to. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString UserToGrant;

};


/* Command messages
 *****************************************************************************/

/**
 * Implements a message for executing a console command.
 */
USTRUCT()
struct FEngineServiceExecuteCommand
{
	GENERATED_USTRUCT_BODY()

	/** Holds the command to execute. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Command;

	/** Holds the name of the user that wants to execute the command. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString UserName;

	/** Default constructor. */
	FEngineServiceExecuteCommand() { }

	/** Creates and initializes a new instance. */
	FEngineServiceExecuteCommand(const FString& InCommand, const FString& InUserName)
		: Command(InCommand)
		, UserName(InUserName)
	{ }
};


/**
 * Implements a message for terminating the engine.
 */
USTRUCT()
struct FEngineServiceTerminate
{
	GENERATED_USTRUCT_BODY()

	/** Holds the name of the user that wants to terminate the engine. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString UserName;

	/** Default constructor. */
	FEngineServiceTerminate() { }

	/** Creates and initializes a new instance. */
	FEngineServiceTerminate(const FString& InUserName)
		: UserName(InUserName)
	{ }
};


/* Status messages
 *****************************************************************************/

/**
 * Implements a message that contains a notification or log output.
 */
USTRUCT()
struct FEngineServiceNotification
{
	GENERATED_USTRUCT_BODY()

	/** Holds the notification text. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Text;

	/** Holds the time in seconds since the engine started. */
	UPROPERTY(EditAnywhere, Category="Message")
	double TimeSeconds;

	/** Default constructor. */
	FEngineServiceNotification() : TimeSeconds(0.0) { }

	/** Creates and initializes a new instance. */
	FEngineServiceNotification(const FString& InText, double InTimeSeconds)
		: Text(InText)
		, TimeSeconds(InTimeSeconds)
	{ }
};
