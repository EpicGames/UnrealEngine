// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Variant.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Containers/BitArray.h"
#include "TraceControlMessages.generated.h"

/* Trace messages
 *****************************************************************************/

/**
 * Request a status update
 */
USTRUCT()
struct FTraceControlStatusPing
{
	GENERATED_USTRUCT_BODY()
};

/**
 * Response for FTraceControlStatusPing. Meant to be fairly high
 * frequency.
 */
USTRUCT()
struct FTraceControlStatus
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	FString Endpoint;
	
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid SessionGuid;
	
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid TraceGuid;

	UPROPERTY(EditAnywhere, Category="Message")
	uint64 BytesSent = 0;

	UPROPERTY(EditAnywhere, Category="Message")
	uint64 BytesTraced = 0;

	UPROPERTY(EditAnywhere, Category="Message")
	uint64 MemoryUsed = 0;

	UPROPERTY(EditAnywhere, Category="Message")
	uint32 CacheAllocated = 0; 

	UPROPERTY(EditAnywhere, Category="Message")
	uint32 CacheUsed = 0;

	UPROPERTY(EditAnywhere, Category="Message")
	uint32 CacheWaste = 0;

	UPROPERTY(EditAnywhere, Category = "Message")
	bool bAreStatNamedEventsEnabled = false;

	UPROPERTY(EditAnywhere, Category = "Message")
	bool bIsPaused = false;

	UPROPERTY(EditAnywhere, Category = "Message")
	bool bIsTracing = false;

	UPROPERTY(EditAnywhere, Category = "Message")
	FDateTime StatusTimestamp;

	UPROPERTY(EditAnywhere, Category = "Message")
	uint8 TraceSystemStatus = 0;
};

/**
 * Request update of settings
 */
USTRUCT()
struct FTraceControlSettingsPing
{
	GENERATED_USTRUCT_BODY()
};

/**
 *	Contains data about a trace channel preset
 */
USTRUCT()
struct FTraceChannelPreset
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Message")
	FString Name;

	UPROPERTY(EditAnywhere, Category = "Message")
	FString ChannelList;

	UPROPERTY(EditAnywhere, Category = "Message")
	bool bIsReadOnly = false;
};

USTRUCT()
struct FTraceCapability
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	FString Name;
};

/**
 * Response for FTraceControlSettingsPing containing
 * TraceLog and system settings.
 */
USTRUCT()
struct FTraceControlSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	bool bUseWorkerThread = false;
	
	UPROPERTY(EditAnywhere, Category="Message")
	bool bUseImportantCache = false;

	UPROPERTY(EditAnywhere, Category="Message")
	uint32 TailSizeBytes = 0;

	UPROPERTY(EditAnywhere, Category = "Message")
	TArray<FTraceChannelPreset> ChannelPresets;

	UPROPERTY(EditAnywhere, Category = "Message")
	TArray<FTraceCapability> Capabilities;
};

/**
 * Request update of channels and status
 */
USTRUCT()
struct FTraceControlChannelsPing
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category="Message")
	uint32 KnownChannelCount = 0;
};

/**
 * Response for FTraceControlChannelsPing with information
 * about available channels.
 */
USTRUCT()
struct FTraceControlChannelsDesc
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category="Message")
	TArray<FString> Channels;

	UPROPERTY(EditAnywhere, Category="Message")
	TArray<uint32> Ids;
	
	UPROPERTY(EditAnywhere, Category="Message")
	TArray<FString> Descriptions;
	
	UPROPERTY(EditAnywhere, Category="Message")
	TArray<uint32> ReadOnlyIds;
};

/**
 * Response for FTraceControlChannelsPing with information about
 * enabled channels.
 */
USTRUCT()
struct FTraceControlChannelsStatus
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category="Message")
	TArray<uint32> EnabledIds;
};


/**
 * Command to set the state of channels
 */
USTRUCT()
struct FTraceControlChannelsSet
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category="Message")
	TArray<uint32> ChannelIdsToEnable;
	
	UPROPERTY(EditAnywhere, Category="Message")
	TArray<uint32> ChannelIdsToDisable;
};

/**
 * Error messages related to channel enable/disable commands
 */
USTRUCT()
struct FTraceControlChannelsSetError
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	TMap<uint32, FString> Errors;
};

/**
 * Message sent when first connecting to a session. Expected to return
 * FTraceControlPong message.
 */
USTRUCT()
struct FTraceControlDiscoveryPing
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid SessionId;
	
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid InstanceId;
};

/**
 * Response for FTraceControlPing. A status message with additional
 * information about session instance and properties that doesn't change
 * over the course of the session lifetime.
 */
USTRUCT()
struct FTraceControlDiscovery : public FTraceControlStatus
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid SessionId;
	
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid InstanceId;
};

/**
 * Stop current trace.
 */
USTRUCT()
struct FTraceControlStop
{
	GENERATED_USTRUCT_BODY()
};

/**
 * Shared properties for starting a trace. Never use this message directly.
 */
USTRUCT()
struct FTraceControlStartCommon
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category="Message")
	FString Channels;

	UPROPERTY(EditAnywhere, Category="Message")
	bool bExcludeTail = false;
};

/**
 * Start tracing to a host
 */
USTRUCT()
struct FTraceControlSend : public FTraceControlStartCommon
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	FString Host;
};

/**
 * Start tracing to a file (on the receiver)
 */
USTRUCT()
struct FTraceControlFile : public FTraceControlStartCommon
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category="Message")
	FString File;
	
	UPROPERTY(EditAnywhere, Category="Message")
	bool bTruncateFile = false;
};

/**
 * Start tracing to a host using a secure connection (if supported on the target)
 */
USTRUCT()
struct FTraceControlSecure : public FTraceControlStartCommon
{
	GENERATED_USTRUCT_BODY()
};


/**
 * Secure connection mode.
 * See TraceAuxiliary for details.
 */
UENUM()
enum class ETraceSecureMode : uint8
{
	SelfSignedCertAndAuthToken,
	CertFileAndAuthToken,
	AuthToken
};

/**
 * Response to a secure start request. Specifies port to connect
 * to, optional authorization token, and optional certificate if
 * using self-signed certificates.
 */
USTRUCT()
struct FTraceControlSecureResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	FString Host;

	UPROPERTY(EditAnywhere, Category="Message")
	uint16 Port = 0;

	UPROPERTY(EditAnywhere, Category="Message")
	ETraceSecureMode Mode = ETraceSecureMode::SelfSignedCertAndAuthToken;

	UPROPERTY(EditAnywhere, Category="Message")
	TArray<uint8> AuthorizationToken;

	UPROPERTY(EditAnywhere, Category="Message")
	TArray<uint8> CertificateAuthority;

	UPROPERTY(EditAnywhere, Category="Message")
	FString Identity;
};



/**
 * Pause all tracing (i.e. mute all channels)
 */
USTRUCT()
struct FTraceControlPause
{
	GENERATED_USTRUCT_BODY()
};

/**
 * Resume all tracing (from pause)
 */
USTRUCT()
struct FTraceControlResume
{
	GENERATED_USTRUCT_BODY()
};

/**
 * Trace a snapshot to the trace server.
 */
USTRUCT()
struct FTraceControlSnapshotSend
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category="Message")
	FString Host;
};

/**
 * Trace a snapshot to a file.
 */
USTRUCT()
struct FTraceControlSnapshotFile
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category="Message")
	FString File;
};

/**
 * Trace a bookmark with the given label.
 */
USTRUCT()
struct FTraceControlBookmark
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	FString Label;
};

/**
 * Trace a screenshot with the given label.
 */
USTRUCT()
struct FTraceControlScreenshot
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category="Message")
	FString Name;
	
	UPROPERTY(EditAnywhere, Category="Message")
	bool bShowUI = true;
};

/**
 * Set the StatNamedEvents flag.
 */
USTRUCT()
struct FTraceControlSetStatNamedEvents
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Message")
	bool bEnabled = false;
};
